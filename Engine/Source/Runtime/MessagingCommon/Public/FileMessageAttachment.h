// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "IMessageAttachment.h"

/**
 * Implements a message attachment whose data is held in a file.
 *
 * WARNING: Message attachments do not work yet for out of process messages.
 */
class FFileMessageAttachment
	: public IMessageAttachment
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param Filename The full name and path of the file.
	 */
	FFileMessageAttachment( const FString& InFilename )
		: AutoDeleteFile(false)
		, Filename(InFilename)
	{ }

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param Filename The full name and path of the file.
	 * @param AutoDeleteFile Whether to delete the file when this attachment is destroyed.
	 */
	FFileMessageAttachment( const FString& InFilename, bool InAutoDeleteFile )
		: AutoDeleteFile(InAutoDeleteFile)
		, Filename(InFilename)
	{ }

	/** Destructor. */
	~FFileMessageAttachment()
	{
		if (AutoDeleteFile)
		{
			IFileManager::Get().Delete(*Filename);
		}
	}

public:

	// IMessageAttachment interface

	virtual FArchive* CreateReader() override
	{
		return IFileManager::Get().CreateFileReader(*Filename);
	}

private:

	/** Holds a flag indicating whether the file should be deleted. */
	bool AutoDeleteFile;

	/** Holds the name of the file that holds the attached data. */
	FString Filename;
};
