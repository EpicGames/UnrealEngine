// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMessageContext.h"
#include "IMessageBridge.h"
#include "IMessageBus.h"
#include "IMessageTransport.h"
#include "IMessagingModule.h"


/**
 * Implements a message bridge builder.
 */
class FMessageBridgeBuilder
{
public:

	/** Default constructor. */
	FMessageBridgeBuilder()
		: Address(FMessageAddress::NewAddress())
		, BusPtr(IMessagingModule::Get().GetDefaultBus())
		, Disabled(false)
		, Transport(nullptr)
	{ }

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InBus The message bus to attach the bridge to.
	 */
	FMessageBridgeBuilder(const TSharedRef<IMessageBus, ESPMode::ThreadSafe>& Bus)
		: Address(FMessageAddress::NewAddress())
		, BusPtr(Bus)
		, Disabled(false)
		, Transport(nullptr)
	{ }

public:

	/**
	 * Disables the bridge.
	 *
	 * @return This instance (for method chaining).
	 */
	FMessageBridgeBuilder& ThatIsDisabled()
	{
		Disabled = true;

		return *this;
	}

	/**
	 * Configures the bridge to use a specific message transport technology.
	 *
	 * @param InTransport The transport technology to use.
	 * @return This instance (for method chaining).
	 */
	FMessageBridgeBuilder& UsingTransport(const TSharedRef<IMessageTransport, ESPMode::ThreadSafe>& InTransport)
	{
		Transport = InTransport;

		return *this;
	}

	/**
	 * Sets the bridge's address.
	 *
	 * If no address is specified, one will be generated automatically.
	 *
	 * @param InAddress The address to set.
	 * @return This instance (for method chaining).
	 */
	FMessageBridgeBuilder& WithAddress(const FMessageAddress& InAddress)
	{
		Address = InAddress;

		return *this;
	}

public:

	/**
	 * Builds the message bridge as configured.
	 *
	 * @return A new message bridge, or nullptr if it couldn't be built.
	 */
	TSharedPtr<IMessageBridge, ESPMode::ThreadSafe> Build()
	{
		TSharedPtr<IMessageBridge, ESPMode::ThreadSafe> Bridge;

		check(Transport.IsValid());

		auto Bus = BusPtr.Pin();

		if (Bus.IsValid())
		{
			Bridge = IMessagingModule::Get().CreateBridge(Address, Bus.ToSharedRef(), Transport.ToSharedRef());

			if (Bridge.IsValid())
			{
				if (Disabled)
				{
					Bridge->Disable();
				}
				else
				{
					Bridge->Enable();
				}
			}
		}

		return Bridge;
	}

	/**
	 * Implicit conversion operator to build the message bridge as configured.
	 *
	 * @return A new message bridge, or nullptr if it couldn't be built.
	 */
	operator TSharedPtr<IMessageBridge, ESPMode::ThreadSafe>()
	{
		return Build();
	}

private:

	/** Holds the bridge's address. */
	FMessageAddress Address;

	/** Holds a weak pointer to the message bus to attach to. */
	TWeakPtr<IMessageBus, ESPMode::ThreadSafe> BusPtr;

	/** Holds a flag indicating whether the bridge should be disabled. */
	bool Disabled;

	/** Holds a reference to the message transport technology. */
	TSharedPtr<IMessageTransport, ESPMode::ThreadSafe> Transport;
};
