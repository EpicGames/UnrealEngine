// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Containers/ArrayBuilder.h"
#include "Containers/Queue.h"
#include "IMessageAttachment.h"
#include "IMessageBus.h"
#include "IMessageContext.h"
#include "IMessageHandler.h"
#include "IMessageReceiver.h"
#include "IMessageSender.h"
#include "IMessageBusListener.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Misc/ScopeLock.h"


/**
 * DEPRECATED: Delegate type for error notifications.
 *
 * The first parameter is the context of the sent message (only valid for the duration of the callback).
 * The second parameter is the error string.
 */
DECLARE_DELEGATE_TwoParams(FOnMessageEndpointError, const IMessageContext&, const FString&);


/**
 * Struct to propagate message bus notifications
 */
struct FMessageBusNotification
{
	/** Notification type. */
	EMessageBusNotification NotificationType;

	/** Address of the un/registered. */
	FMessageAddress RegistrationAddress;
};

/** Delegate type for MessageBus notifications. */
DECLARE_DELEGATE_OneParam(FOnBusNotification, const FMessageBusNotification&);

/**
 * Implements a message endpoint for sending and receiving messages on a message bus.
 *
 * This class provides a convenient implementation of the IMessageReceiver and IMessageSender interfaces,
 * which allow consumers to send and receive messages on a message bus. The endpoint allows for receiving
 * messages asynchronously as they arrive, as well as synchronously through an inbox that can be polled.
 *
 * By default, messages are received synchronously on the thread that the endpoint was created on.
 * If the message consumer is thread-safe, a more efficient message dispatch can be enabled by calling
 * the SetRecipientThread() method with ENamedThreads::AnyThread.
 *
 * Endpoints that are destroyed or receive messages on non-Game threads should use the static function
 * FMessageEndpoint::SafeRelease() to dispose of the endpoint. This will ensure that there are no race
 * conditions between endpoint destruction and the receiving of messages.
 *
 * The underlying message bus will take ownership of all sent and published message objects. The memory
 * held by the messages must therefore NOT be freed by the caller.
 */
class FMessageEndpoint
	: public TSharedFromThis<FMessageEndpoint, ESPMode::ThreadSafe>
	, public IMessageReceiver
	, public IMessageSender
	, public IBusListener
{
public:

	/**
	 * Type definition for the endpoint builder.
	 *
	 * When building message endpoints that receive messages on AnyThread, use the SafeRelease
	 * helper function to avoid race conditions when destroying the objects that own the endpoints.
	 *
	 * @see SafeRelease
	 */
	typedef struct FMessageEndpointBuilder Builder;

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InName The endpoint's name (for debugging purposes).
	 * @param InBus The message bus to attach this endpoint to.
	 * @param InHandlers The collection of message handlers to register.
	 */
	FMessageEndpoint(const FName& InName, const TSharedRef<IMessageBus, ESPMode::ThreadSafe>& InBus, const TArray<TSharedPtr<IMessageHandler, ESPMode::ThreadSafe>>& InHandlers, const FOnBusNotification InNotificationDelegate)
		: Address(FMessageAddress::NewAddress())
		, BusPtr(InBus)
		, Enabled(true)
		, Handlers(InHandlers)
		, NotificationDelegate(InNotificationDelegate)
		, Id(FGuid::NewGuid())
		, InboxEnabled(false)
		, Name(InName)
	{
		SetRecipientThread(FTaskGraphInterface::Get().GetCurrentThreadIfKnown());
	}

	/** Destructor. */
	~FMessageEndpoint()
	{	
		auto Bus = BusPtr.Pin();

		if (Bus.IsValid())
		{
			Bus->Unregister(Address);
		}
	}

public:

	/**
	 * Disables this endpoint.
	 *
	 * A disabled endpoint will not receive any subscribed messages until it is enabled again.
	 * Endpoints should be created in an enabled state by default.
	 *
	 * @see Enable, IsEnabled
	 */
	void Disable()
	{
		Enabled = false;
	}

	/**
	 * Enables this endpoint.
	 *
	 * An activated endpoint will receive subscribed messages.
	 * Endpoints should be created in an enabled state by default.
	 *
	 * @see Disable, IsEnabled
	 */
	void Enable()
	{
		Enabled = true;
	}

	/**
	 * Gets the endpoint's message address.
	 *
	 * @return Message address.
	 */
	const FMessageAddress& GetAddress() const
	{
		return Address;
	}

	/**
	 * Checks whether this endpoint is connected to the bus.
	 *
	 * @return true if connected, false otherwise.
	 * @see IsEnabled
	 */
	bool IsConnected() const
	{
		return BusPtr.IsValid();
	}

	/**
	 * Checks whether this endpoint is enabled.
	 *
	 * @return true if the endpoint is enabled, false otherwise.
	 * @see Disable, Enable, IsConnected
	 */
	bool IsEnabled() const
	{
		return Enabled;
	}

	/**
	 * Sets the name of the thread to receive messages on.
	 *
	 * Use this method to receive messages on a particular thread, for example, if the
	 * consumer owning this endpoint is not thread-safe. The default value is ThreadAny.
	 *
	 * ThreadAny is the fastest way to receive messages. It should be used if the receiving
	 * code is completely thread-safe and if it is sufficiently fast. ThreadAny MUST NOT
	 * be used if the receiving code is not thread-safe. It also SHOULD NOT be used if the
	 * code includes time consuming operations, because it will block the message router,
	 * causing no other messages to be delivered in the meantime.
	 *
	 * @param NamedThread The name of the thread to receive messages on.
	 */
	void SetRecipientThread(const ENamedThreads::Type& NamedThread)
	{
		RecipientThread = ENamedThreads::GetThreadIndex(NamedThread);
	}

public:

	/**
	 * Defers processing of the given message by the specified time delay.
	 *
	 * The message is effectively delivered again to this endpoint after the
	 * original sent time plus the time delay have elapsed.
	 *
	 * @param Context The context of the message to defer.
	 * @param Delay The time delay.
	 */
	void Defer(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const FTimespan& Delay)
	{
		TSharedPtr<IMessageBus, ESPMode::ThreadSafe> Bus = GetBusIfEnabled();

		if (Bus.IsValid())
		{
			Bus->Forward(Context, TArrayBuilder<FMessageAddress>().Add(Address), Delay, AsShared());
		}
	}

	/**
	 * Forwards a previously received message.
	 *
	 * Messages can only be forwarded to endpoints within the same process.
	 *
	 * @param Context The context of the message to forward.
	 * @param Recipients The list of message recipients to forward the message to.
	 * @param Delay The time delay.
	 */
	void Forward(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const TArray<FMessageAddress>& Recipients, const FTimespan& Delay)
	{
		TSharedPtr<IMessageBus, ESPMode::ThreadSafe> Bus = GetBusIfEnabled();

		if (Bus.IsValid())
		{
			Bus->Forward(Context, Recipients, Delay, AsShared());
		}
	}

	/**
	 * Publishes a message to all subscribed recipients within the specified scope.
	 *
	 * @param Message The message to publish.
	 * @param TypeInfo The message's type information.
	 * @param Scope The message scope.
	 * @param Fields The message content.
	 * @param Delay The delay after which to publish the message.
	 * @param Expiration The time at which the message expires.
	 */
	void Publish(void* Message, UScriptStruct* TypeInfo, EMessageScope Scope, const FTimespan& Delay, const FDateTime& Expiration)
	{
		Publish(Message, TypeInfo, Scope, TMap<FName, FString>(), Delay, Expiration);
	}

	/**
	 * Publishes a message to all subscribed recipients within the specified scope.
	 *
	 * @param Message The message to publish.
	 * @param TypeInfo The message's type information.
	 * @param Scope The message scope.
	 * @param Annotations An optional message annotations header.
	 * @param Fields The message content.
	 * @param Delay The delay after which to publish the message.
	 * @param Expiration The time at which the message expires.
	 */
	void Publish(void* Message, UScriptStruct* TypeInfo, EMessageScope Scope, const TMap<FName, FString> Annotations, const FTimespan& Delay, const FDateTime& Expiration)
	{
		TSharedPtr<IMessageBus, ESPMode::ThreadSafe> Bus = GetBusIfEnabled();

		if (Bus.IsValid())
		{
			Bus->Publish(Message, TypeInfo, Scope, Annotations, Delay, Expiration, AsShared());
		}
	}

	/**
	 * Sends a message to the specified list of recipients.
	 *
	 * @param Message The message to send.
	 * @param TypeInfo The message's type information.
	 * @param Attachment An optional binary data attachment.
	 * @param Recipients The message recipients.
	 * @param Delay The delay after which to send the message.
	 * @param Expiration The time at which the message expires.
	 */
	UE_DEPRECATED(4.21, "FMessageEndpoint::Send with 6 params is deprecated. Please use FMessageEndpoint::Send that takes additionnal EMessageFlags instead!")
	void Send(void* Message, UScriptStruct* TypeInfo, const TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe>& Attachment, const TArray<FMessageAddress>& Recipients, const FTimespan& Delay, const FDateTime& Expiration)
	{
		Send(Message, TypeInfo, EMessageFlags::None, Attachment, Recipients, Delay, Expiration);
	}

	/**
	 * Sends a message to the specified list of recipients.
	 * Allows to specify message flags
	 *
	 * @param Message The message to send.
	 * @param TypeInfo The message's type information.
	 * @param Flags The message's type information.
	 * @param Attachment An optional binary data attachment.
	 * @param Recipients The message recipients.
	 * @param Delay The delay after which to send the message.
	 * @param Expiration The time at which the message expires.
	 */
	void Send(void* Message, UScriptStruct* TypeInfo, EMessageFlags Flags, const TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe>& Attachment, const TArray<FMessageAddress>& Recipients, const FTimespan& Delay, const FDateTime& Expiration)
	{
		TSharedPtr<IMessageBus, ESPMode::ThreadSafe> Bus = GetBusIfEnabled();

		if (Bus.IsValid())
		{
			Bus->Send(Message, TypeInfo, Flags, TMap<FName, FString>(), Attachment, Recipients, Delay, Expiration, AsShared());
		}
	}

	/**
	 * Sends a message to the specified list of recipients.
	 *
	 * @param Message The message to send.
	 * @param TypeInfo The message's type information.
	 * @param Flags The message's type information.
	 * @param Annotations An optional message annotations header.
	 * @param Attachment An optional binary data attachment.
	 * @param Recipients The message recipients.
	 * @param Delay The delay after which to send the message.
	 * @param Expiration The time at which the message expires.
	 */
	void Send(void* Message, UScriptStruct* TypeInfo, EMessageFlags Flags, const TMap<FName, FString> Annotations, const TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe>& Attachment, const TArray<FMessageAddress>& Recipients, const FTimespan& Delay, const FDateTime& Expiration)
	{
		TSharedPtr<IMessageBus, ESPMode::ThreadSafe> Bus = GetBusIfEnabled();

		if (Bus.IsValid())
		{
			Bus->Send(Message, TypeInfo, Flags, Annotations, Attachment, Recipients, Delay, Expiration, AsShared());
		}
	}

	/**
	 * Subscribes a message handler.
	 *
	 * @param MessageType The type name of the messages to subscribe to.
	 * @param ScopeRange The range of message scopes to include in the subscription.
	 */
	UE_DEPRECATED(5.1, "Types names are now represented by path names. Please use a version of this function that takes an FTopLevelAssetPath as MessageType.")
	void Subscribe(const FName& MessageType, const FMessageScopeRange& ScopeRange)
	{
		Subscribe(UClass::TryConvertShortTypeNameToPathName<UStruct>(MessageType.ToString()), ScopeRange);
	}

	/**
	 * Subscribes a message handler.
	 *
	 * @param MessageType The type name of the messages to subscribe to.
	 * @param ScopeRange The range of message scopes to include in the subscription.
	 */
	void Subscribe(const FTopLevelAssetPath& MessageType, const FMessageScopeRange& ScopeRange)
	{
		TSharedPtr<IMessageBus, ESPMode::ThreadSafe> Bus = GetBusIfEnabled();

		if (Bus.IsValid())
		{
			Bus->Subscribe(AsShared(), MessageType, ScopeRange);
		}
	}

	/**
	 * Unsubscribes this endpoint from the specified message type.
	 *
	 * @param MessageType The type of message to unsubscribe (IMessageBus::PATHNAME_All = all types).
	 * @see Subscribe
	 */
	UE_DEPRECATED(5.1, "Types names are now represented by path names. Please use a version of this function that takes an FTopLevelAssetPath as MessageType.")
	void Unsubscribe(const FName& MessageType)
	{
		Unsubscribe(UClass::TryConvertShortTypeNameToPathName<UStruct>(MessageType.ToString()));
	}

	/**
	 * Unsubscribes this endpoint from the specified message type.
	 *
	 * @param MessageType The type of message to unsubscribe (IMessageBus::PATHNAME_All = all types).
	 * @see Subscribe
	 */
	void Unsubscribe(const FTopLevelAssetPath& MessageType)
	{
		TSharedPtr<IMessageBus, ESPMode::ThreadSafe> Bus = GetBusIfEnabled();

		if (Bus.IsValid())
		{
			Bus->Unsubscribe(AsShared(), MessageType);
		}
	}

public:

	/**
	 * Disables the inbox for unhandled messages.
	 *
	 * The inbox is disabled by default.
	 *
	 * @see EnableInbox, IsInboxEmpty, IsInboxEnabled, ProcessInbox, ReceiveFromInbox
	 */
	void DisableInbox()
	{
		InboxEnabled = false;
	}

	/**
	 * Enables the inbox for unhandled messages.
	 *
	 * If enabled, the inbox will queue up all received messages. Use ProcessInbox() to synchronously
	 * invoke the registered message handlers for all queued up messages, or ReceiveFromInbox() to
	 * manually receive one message from the inbox at a time. The inbox is disabled by default.
	 *
	 * @see DisableInbox, IsInboxEmpty, IsInboxEnabled, ProcessInbox, ReceiveFromInbox
	 */
	void EnableInbox()
	{
		InboxEnabled = true;
	}

	/**
	 * Checks whether the inbox is empty.
	 *
	 * @return true if the inbox is empty, false otherwise.
	 * @see DisableInbox, EnableInbox, IsInboxEnabled, ProcessInbox, ReceiveFromInbox
	 */
	bool IsInboxEmpty() const
	{
		return Inbox.IsEmpty();
	}

	/**
	 * Checks whether the inbox is enabled.
	 *
	 * @see DisableInbox, EnableInbox, IsInboxEmpty, ProcessInbox, ReceiveFromInbox
	 */
	bool IsInboxEnabled() const
	{
		return InboxEnabled;
	}

	/**
	 * Calls the matching message handlers for all messages queued up in the inbox.
	 *
	 * Note that an incoming message will only be queued up in the endpoint's inbox if the inbox has
	 * been enabled and no matching message handler handled it. The inbox is disabled by default and
	 * must be enabled using the EnableInbox() method.
	 *
	 * @see IsInboxEmpty, ReceiveFromInbox
	 */
	void ProcessInbox()
	{
		TSharedPtr<IMessageContext, ESPMode::ThreadSafe> Context;

		while (Inbox.Dequeue(Context))
		{
			ProcessMessage(Context.ToSharedRef());
		}
	}

	/**
	 * Receives a single message from the endpoint's inbox.
	 *
	 * Note that an incoming message will only be queued up in the endpoint's inbox if the inbox has
	 * been enabled and no matching message handler handled it. The inbox is disabled by default and
	 * must be enabled using the EnableInbox() method.
	 *
	 * @param OutContext Will hold the context of the received message.
	 * @return true if a message was received, false if the inbox was empty.
	 * @see DisableInbox, EnableInbox, IsInboxEnabled, ProcessInbox
	 */
	bool ReceiveFromInbox(TSharedPtr<IMessageContext, ESPMode::ThreadSafe>& OutContext)
	{
		return Inbox.Dequeue(OutContext);
	}

public:

	//~ IMessageReceiver interface

	virtual FName GetDebugName() const override
	{
		return Name;
	}

	virtual const FGuid& GetRecipientId() const override
	{
		return Id;
	}

	virtual ENamedThreads::Type GetRecipientThread() const override
	{
		return RecipientThread;
	}

	virtual bool IsLocal() const override
	{
		return true;
	}

	virtual void ReceiveMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context) override
	{
		if (!Enabled)
		{
			return;
		}

		if (InboxEnabled)
		{
			Inbox.Enqueue(Context);
		}
		else
		{
			ProcessMessage(Context);
		}
	}

	//~ IBusListener interface

	virtual ENamedThreads::Type GetListenerThread() const override
	{
		return RecipientThread;
	}

	virtual void NotifyRegistration(const FMessageAddress& InAddress, EMessageBusNotification InNotification)
	{
		if (!Enabled)
		{
			return;
		}

		NotificationDelegate.ExecuteIfBound(FMessageBusNotification{ InNotification, InAddress });
	}

public:

	//~ IMessageSender interface

	virtual FMessageAddress GetSenderAddress() override
	{
		return Address;
	}

	virtual void NotifyMessageError(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const FString& Error) override
	{
		ErrorDelegate.ExecuteIfBound(Context.Get(), Error);
	}

public:

	/**
	 * Creates a message of the specified template type.
	 *
	 * Prefer using this helper rather than explicit new. See FEngineServicePong.
	 *
	 * @param Args The constructor arguments to the template type
	 */
	template<typename T, typename... InArgTypes>
	static T* MakeMessage(InArgTypes&&... Args)
	{
		void* Buffer = FMemory::Malloc(sizeof(T));

		T* Message = new (Buffer) T(::Forward<InArgTypes>(Args)...);

		return Message;
	}

	/**
	 * Immediately forwards a previously received message to the specified recipient.
	 *
	 * Messages can only be forwarded to endpoints within the same process.
	 *
	 * @param Context The context of the message to forward.
	 * @param Recipient The address of the recipient to forward the message to.
	 */
	void Forward(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const FMessageAddress& Recipient)
	{
		Forward(Context, TArrayBuilder<FMessageAddress>().Add(Recipient), FTimespan::Zero());
	}

	/**
	 * Forwards a previously received message to the specified recipient after a given delay.
	 *
	 * Messages can only be forwarded to endpoints within the same process.
	 *
	 * @param Context The context of the message to forward.
	 * @param Recipient The address of the recipient to forward the message to.
	 * @param ForwardingScope The scope of the forwarded message.
	 * @param Delay The delay after which to publish the message.
	 */
	void Forward(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const FMessageAddress& Recipient, const FTimespan& Delay)
	{
		Forward(Context, TArrayBuilder<FMessageAddress>().Add(Recipient), Delay);
	}

	/**
	 * Immediately forwards a previously received message to the specified list of recipients.
	 *
	 * Messages can only be forwarded to endpoints within the same process.
	 *
	 * @param Context The context of the message to forward.
	 * @param Recipients The list of message recipients to forward the message to.
	 * @param ForwardingScope The scope of the forwarded message.
	 */
	void Forward(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const TArray<FMessageAddress>& Recipients)
	{
		Forward(Context, Recipients, FTimespan::Zero());
	}

	/**
	 * Immediately publishes a message to all subscribed recipients.
	 *
	 * @param Message The message to publish.
	 */
	template<typename MessageType>
	void Publish(MessageType* Message)
	{
		Publish(Message, MessageType::StaticStruct(), EMessageScope::Network, FTimespan::Zero(), FDateTime::MaxValue());
	}

	/**
	 * Immediately pa message to all subscribed recipients within the specified scope.
	 *
	 * @param Message The message to publish.
	 * @param Scope The message scope.
	 */
	template<typename MessageType>
	void Publish(MessageType* Message, EMessageScope Scope)
	{
		Publish(Message, MessageType::StaticStruct(), Scope, FTimespan::Zero(), FDateTime::MaxValue());
	}

	/**
	 * Immediately publishes a message to all subscribed recipients.
	 *
	 * @param Message The message to publish.
	 * @param Annotations An optional message annotations header.
	 */
	template<typename MessageType>
	void Publish(MessageType* Message, const TMap<FName, FString> Annotations)
	{
		Publish(Message, MessageType::StaticStruct(), EMessageScope::Network, Annotations, FTimespan::Zero(), FDateTime::MaxValue());
	}

	/**
	 * Immediately pa message to all subscribed recipients within the specified scope.
	 *
	 * @param Message The message to publish.
	 * @param Annotations An optional message annotations header.
	 * @param Scope The message scope.
	 */
	template<typename MessageType>
	void Publish(MessageType* Message, const TMap<FName, FString> Annotations, EMessageScope Scope)
	{
		Publish(Message, MessageType::StaticStruct(), Scope, Annotations, FTimespan::Zero(), FDateTime::MaxValue());
	}

	/**
	 * Publishes a message to all subscribed recipients after a given delay.
	 *
	 * @param Message The message to publish.
	 * @param Delay The delay after which to publish the message.
	 */
	template<typename MessageType>
	void Publish(MessageType* Message, const FTimespan& Delay)
	{
		Publish(Message, MessageType::StaticStruct(), EMessageScope::Network, Delay, FDateTime::MaxValue());
	}

	/**
	 * Publishes a message to all subscribed recipients within the specified scope after a given delay.
	 *
	 * @param Message The message to publish.
	 * @param Scope The message scope.
	 * @param Delay The delay after which to publish the message.
	 */
	template<typename MessageType>
	void Publish(MessageType* Message, EMessageScope Scope, const FTimespan& Delay)
	{
		Publish(Message, MessageType::StaticStruct(), Scope, Delay, FDateTime::MaxValue());
	}

	/**
	 * Publishes a message to all subscribed recipients within the specified scope.
	 *
	 * @param Message The message to publish.
	 * @param Scope The message scope.
	 * @param Fields The message content.
	 * @param Delay The delay after which to publish the message.
	 * @param Expiration The time at which the message expires.
	 */
	template<typename MessageType>
	void Publish(MessageType* Message, EMessageScope Scope, const FTimespan& Delay, const FDateTime& Expiration)
	{
		Publish(Message, MessageType::StaticStruct(), Scope, Delay, Expiration);
	}

	/**
	 * Publishes a message to all subscribed recipients within the specified scope.
	 *
	 * @param Message The message to publish.
	 * @param Annotations An optional message annotations header.
	 * @param Scope The message scope.
	 * @param Fields The message content.
	 * @param Delay The delay after which to publish the message.
	 * @param Expiration The time at which the message expires.
	 */
	template<typename MessageType>
	void Publish(MessageType* Message, const TMap<FName, FString> Annotations, EMessageScope Scope, const FTimespan& Delay, const FDateTime& Expiration)
	{
		Publish(Message, MessageType::StaticStruct(), Scope, Annotations, Delay, Expiration);
	}

	/**
	 * Immediately sends a message to the specified recipient.
	 *
	 * @param MessageType The type of message to send.
	 * @param Message The message to send.
	 * @param Recipient The message recipient.
	 */
	template<typename MessageType>
	void Send(MessageType* Message, const FMessageAddress& Recipient)
	{
		Send(Message, MessageType::StaticStruct(), EMessageFlags::None, nullptr, TArrayBuilder<FMessageAddress>().Add(Recipient), FTimespan::Zero(), FDateTime::MaxValue());
	}

	/**
	 * Immediately sends a message to the specified recipient.
	 *
	 * @param MessageType The type of message to send.
	 * @param Message The message to send.
	 * @param Annotations An optional message annotations header.
	 * @param Recipient The message recipient.
	 */
	template<typename MessageType>
	void Send(MessageType* Message, const TMap<FName, FString> Annotations, const FMessageAddress& Recipient)
	{
		Send(Message, MessageType::StaticStruct(), EMessageFlags::None, Annotations, nullptr, TArrayBuilder<FMessageAddress>().Add(Recipient), FTimespan::Zero(), FDateTime::MaxValue());
	}

	/**
	 * Sends a message to the specified recipient after a given delay.
	 *
	 * @param MessageType The type of message to send.
	 * @param Message The message to send.
	 * @param Recipient The message recipient.
	 * @param Delay The delay after which to send the message.
	 */
	template<typename MessageType>
	void Send(MessageType* Message, const FMessageAddress& Recipient, const FTimespan& Delay)
	{
		Send(Message, MessageType::StaticStruct(), EMessageFlags::None, nullptr, TArrayBuilder<FMessageAddress>().Add(Recipient), Delay, FDateTime::MaxValue());
	}

	/**
	 * Sends a message with fields and expiration to the specified recipient after a given delay.
	 *
	 * @param MessageType The type of message to send.
	 * @param Message The message to send.
	 * @param Recipient The message recipient.
	 * @param Expiration The time at which the message expires.
	 * @param Delay The delay after which to send the message.
	 */
	template<typename MessageType>
	void Send(MessageType* Message, const FMessageAddress& Recipient, const FTimespan& Delay, const FDateTime& Expiration)
	{
		Send(Message, MessageType::StaticStruct(), EMessageFlags::None, nullptr, TArrayBuilder<FMessageAddress>().Add(Recipient), Delay, Expiration);
	}

	/**
	 * Sends a message with fields and attachment to the specified recipient.
	 *
	 * @param MessageType The type of message to send.
	 * @param Message The message to send.
	 * @param Attachment An optional binary data attachment.
	 * @param Recipient The message recipient.
	 */
	template<typename MessageType>
	void Send(MessageType* Message, const TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe>& Attachment, const FMessageAddress& Recipient)
	{
		Send(Message, MessageType::StaticStruct(), EMessageFlags::None, Attachment, TArrayBuilder<FMessageAddress>().Add(Recipient), FTimespan::Zero(), FDateTime::MaxValue());
	}

	/**
	 * Sends a message with fields, attachment and expiration to the specified recipient after a given delay.
	 *
	 * @param MessageType The type of message to send.
	 * @param Message The message to send.
	 * @param Attachment An optional binary data attachment.
	 * @param Recipient The message recipient.
	 * @param Expiration The time at which the message expires.
	 * @param Delay The delay after which to send the message.
	 */
	template<typename MessageType>
	void Send(MessageType* Message, const TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe>& Attachment, const FMessageAddress& Recipient, const FDateTime& Expiration, const FTimespan& Delay)
	{
		Send(Message, MessageType::StaticStruct(), EMessageFlags::None, Attachment, TArrayBuilder<FMessageAddress>().Add(Recipient), Delay, Expiration);
	}

	/**
	 * Sends a message with fields, attachment and expiration to the specified recipient after a given delay.
	 *
	 * @param MessageType The type of message to send.
	 * @param Message The message to send.
	 * @param Annotations An optional message annotations header.
	 * @param Attachment An optional binary data attachment.
	 * @param Recipient The message recipient.
	 * @param Expiration The time at which the message expires.
	 * @param Delay The delay after which to send the message.
	 */
	template<typename MessageType>
	void Send(MessageType* Message, const TMap<FName, FString> Annotations, const TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe>& Attachment, const FMessageAddress& Recipient, const FDateTime& Expiration, const FTimespan& Delay)
	{
		Send(Message, MessageType::StaticStruct(), EMessageFlags::None, Annotations, Attachment, TArrayBuilder<FMessageAddress>().Add(Recipient), Delay, Expiration);
	}

	/**
	 * Immediately sends a message to the specified list of recipients.
	 *
	 * @param MessageType The type of message to send.
	 * @param Message The message to send.
	 * @param Recipients The message recipients.
	 */
	template<typename MessageType>
	void Send(MessageType* Message, const TArray<FMessageAddress>& Recipients)
	{
		Send(Message, MessageType::StaticStruct(), EMessageFlags::None, nullptr, Recipients, FTimespan::Zero(), FDateTime::MaxValue());
	}

	/**
	 * Sends a message to the specified list of recipients after a given delay after a given delay.
	 *
	 * @param MessageType The type of message to send.
	 * @param Message The message to send.
	 * @param Recipients The message recipients.
	 * @param Delay The delay after which to send the message.
	 */
	template<typename MessageType>
	void Send(MessageType* Message, const TArray<FMessageAddress>& Recipients, const FTimespan& Delay)
	{
		Send(Message, MessageType::StaticStruct(), EMessageFlags::None, nullptr, Recipients, Delay, FDateTime::MaxValue());
	}

	/**
	 * Sends a message with fields and attachment to the specified list of recipients after a given delay.
	 *
	 * @param MessageType The type of message to send.
	 * @param Message The message to send.
	 * @param Attachment An optional binary data attachment.
	 * @param Recipients The message recipients.
	 * @param Delay The delay after which to send the message.
	 */
	template<typename MessageType>
	void Send(MessageType* Message, const TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe>& Attachment, const TArray<FMessageAddress>& Recipients, const FTimespan& Delay)
	{
		Send(Message, MessageType::StaticStruct(), EMessageFlags::None, Attachment, Recipients, Delay, FDateTime::MaxValue());
	}

	/**
	 * Sends a message to the specified list of recipients.
	 *
	 * @param MessageType The type of message to send.
	 * @param Message The message to send.
	 * @param Attachment An optional binary data attachment.
	 * @param Recipients The message recipients.
	 * @param Delay The delay after which to send the message.
	 * @param Expiration The time at which the message expires.
	 */
	template<typename MessageType>
	void Send(MessageType* Message, const TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe>& Attachment, const TArray<FMessageAddress>& Recipients, const FTimespan& Delay, const FDateTime& Expiration)
	{
		Send(Message, MessageType::StaticStruct(), EMessageFlags::None, Attachment, Recipients, Delay, Expiration);
	}

	/**
	 * Sends a message to the specified list of recipients.
	 * Allows to specify message flags
	 *
	 * @param Message The message to send.
	 * @param TypeInfo The message's type information.
	 * @param Flags The message's type information.
	 * @param Attachment An optional binary data attachment.
	 * @param Recipients The message recipients.
	 * @param Delay The delay after which to send the message.
	 * @param Expiration The time at which the message expires.
	 */
	template<typename MessageType>
	void Send(MessageType* Message, EMessageFlags Flags, const TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe>& Attachment, const TArray<FMessageAddress>& Recipients, const FTimespan& Delay, const FDateTime& Expiration)
	{
		Send(Message, MessageType::StaticStruct(), Flags, Attachment, Recipients, Delay, Expiration);
	}

	/**
	 * Sends a message to the specified list of recipients.
	 * Allows to specify message flags
	 *
	 * @param Message The message to send.
	 * @param TypeInfo The message's type information.
	 * @param Flags The message's type information.
	 * @param Annotations An optional message annotations header.
	 * @param Attachment An optional binary data attachment.
	 * @param Recipients The message recipients.
	 * @param Delay The delay after which to send the message.
	 * @param Expiration The time at which the message expires.
	 */
	template<typename MessageType>
	void Send(MessageType* Message, EMessageFlags Flags, const TMap<FName, FString> Annotations, const TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe>& Attachment, const TArray<FMessageAddress>& Recipients, const FTimespan& Delay, const FDateTime& Expiration)
	{
		Send(Message, MessageType::StaticStruct(), Flags, Annotations, Attachment, Recipients, Delay, Expiration);
	}

	/**
	 * Template method to subscribe the message endpoint to the specified type of messages with the default message scope.
	 *
	 * The default message scope is all messages excluding loopback messages.
	 *
	 * @param HandlerType The type of the class handling the message.
	 * @param MessageType The type of messages to subscribe to.
	 * @param Handler The class handling the messages.
	 * @param HandlerFunc The class function handling the messages.
	 */
	template<class MessageType>
	void Subscribe()
	{
		Subscribe(MessageType::StaticStruct()->GetStructPathName(), FMessageScopeRange::AtLeast(EMessageScope::Thread));
	}

	/**
	 * Template method to subscribe the message endpoint to the specified type and scope of messages.
	 *
	 * @param HandlerType The type of the class handling the message.
	 * @param MessageType The type of messages to subscribe to.
	 * @param Handler The class handling the messages.
	 * @param HandlerFunc The class function handling the messages.
	 * @param ScopeRange The range of message scopes to include in the subscription.
	 */
	template<class MessageType>
	void Subscribe(const FMessageScopeRange& ScopeRange)
	{
		Subscribe(MessageType::StaticStruct()->GetStructPathName(), ScopeRange);
	}

	/**
	 * Unsubscribes this endpoint from all message types.
	 *
	 * @see Subscribe
	 */
	void Unsubscribe()
	{
		Unsubscribe(IMessageBus::PATHNAME_All);
	}

	/**
	 * Template method to unsubscribe the endpoint from the specified message type.
	 *
	 * @param MessageType The type of message to unsubscribe (IMessageBus::PATHNAME_All = all types).
	 * @see Subscribe
	 */
	template<class MessageType>
	void Unsubscribe()
	{
		Unsubscribe(MessageType::StaticStruct()->GetStructPathName());
	}

public:

	/**
	 * Safely releases a message endpoint that is receiving messages on AnyThread.
	 *
	 * When an object that owns a message endpoint receiving on AnyThread is being destroyed,
	 * it is possible that the endpoint can outlive the object for a brief period of time if
	 * the Messaging system is dispatching messages to it. This helper function
	 * is to block the calling thread while any messages are being dispatched, so that the
	 * endpoint does not invoke any message handlers after the object has been destroyed.
	 *
	 * @param Endpoint The message endpoint to release.
	 */
	static void SafeRelease(TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe>& Endpoint)
	{
		if (Endpoint.IsValid())
		{
			Endpoint->ClearHandlers();
			Endpoint.Reset();
		}
	}

protected:

	/**
	 * Clears all handlers in a way that guarantees it won't overlap with message processing. This preserves internal integrity
	 * of the array and cases where our owner may be shutting down while receiving messages.
	*/
	void ClearHandlers()
	{
		FScopeLock Lock(&HandlersCS);
		Handlers.Empty();
	}

	/**
	 * Gets a shared pointer to the message bus if this endpoint is enabled.
	 *
	 * @return The message bus.
	 */
	inline TSharedPtr<IMessageBus, ESPMode::ThreadSafe> GetBusIfEnabled() const
	{
		if (Enabled)
		{
			return BusPtr.Pin();
		}

		return nullptr;
	}

	/**
	 * Forwards the given message context to matching message handlers.
	 *
	 * @param Context The context of the message to handle.
	 */
	void ProcessMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
	{
		if (!Context->IsValid())
		{
			return;
		}

		FScopeLock Lock(&HandlersCS);

		for (int32 HandlerIndex = 0; HandlerIndex < Handlers.Num(); ++HandlerIndex)
		{
			Handlers[HandlerIndex]->HandleMessage(Context);
		}
	}

private:

	/** Holds the endpoint's identifier. */
	const FMessageAddress Address;

	/** Holds a weak pointer to the message bus. */
	TWeakPtr<IMessageBus, ESPMode::ThreadSafe> BusPtr;

	/** Hold a flag indicating whether this endpoint is active. */
	bool Enabled;

	/** Holds the registered message handlers. */
	TArray<TSharedPtr<IMessageHandler, ESPMode::ThreadSafe>> Handlers;

	/** Holds a delegate that is invoked on disconnection events. */
	FOnBusNotification NotificationDelegate;

	/** Holds the endpoint's unique identifier (for debugging purposes). */
	const FGuid Id;

	/** Holds the endpoint's message inbox. */
	TQueue<TSharedPtr<IMessageContext, ESPMode::ThreadSafe>, EQueueMode::Mpsc> Inbox;

	/** Holds a flag indicating whether the inbox is enabled. */
	bool InboxEnabled;

	/** Holds the endpoint's name (for debugging purposes). */
	const FName Name;

	/** Holds the name of the thread on which to receive messages. */
	ENamedThreads::Type RecipientThread;

private:

	/** Holds a delegate that is invoked in case of messaging errors. */
	FOnMessageEndpointError ErrorDelegate;

	/** Sigfnifies that the handler array is being accessed and other threads should wait or skip */
	FCriticalSection		HandlersCS;
};
