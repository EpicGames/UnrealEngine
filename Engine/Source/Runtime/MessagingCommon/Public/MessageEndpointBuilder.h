// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Async/TaskGraphInterfaces.h"
#include "IMessageBus.h"
#include "IMessageHandler.h"
#include "IMessagingModule.h"
#include "MessageEndpoint.h"
#include "MessageHandlers.h"


/**
 * Implements a message endpoint builder.
 */
struct FMessageEndpointBuilder
{
public:

	/**
	 * Creates and initializes a new builder using the default message bus.
	 *
	 * WARNING: This constructor must be called from the Game thread.
	 *
	 * @param InName The endpoint's name (for debugging purposes).
	 * @param InBus The message bus to attach the endpoint to.
	 */
	FMessageEndpointBuilder(const FName& InName)
		: BusPtr(IMessagingModule::Get().GetDefaultBus())
		, Disabled(false)
		, InboxEnabled(false)
		, Name(InName)
		, RecipientThread(FTaskGraphInterface::Get().GetCurrentThreadIfKnown())
	{ }

	/**
	 * Creates and initializes a new builder using the specified message bus.
	 *
	 * @param InName The endpoint's name (for debugging purposes).
	 * @param InBus The message bus to attach the endpoint to.
	 */
	FMessageEndpointBuilder(const FName& InName, const TSharedRef<IMessageBus, ESPMode::ThreadSafe>& InBus)
		: BusPtr(InBus)
		, Disabled(false)
		, InboxEnabled(false)
		, Name(InName)
		, RecipientThread(FTaskGraphInterface::Get().GetCurrentThreadIfKnown())
	{ }

public:

	/**
	 * Adds a message handler for the given type of messages (via raw function pointers).
	 *
	 * It is legal to configure multiple handlers for the same message type. Each
	 * handler will be executed when a message of the specified type is received.
	 *
	 * This overload is used to register raw class member functions.
	 *
	 * @param HandlerType The type of the object handling the messages.
	 * @param MessageType The type of messages to handle.
	 * @param Handler The class handling the messages.
	 * @param HandlerFunc The class function handling the messages.
	 * @return This instance (for method chaining).
	 * @see WithCatchall, WithHandler
	 */
	template<typename MessageType, typename HandlerType>
	FMessageEndpointBuilder& Handling(HandlerType* Handler, typename TRawMessageHandler<MessageType, HandlerType>::FuncType HandlerFunc)
	{
		Handlers.Add(MakeShareable(new TRawMessageHandler<MessageType, HandlerType>(Handler, MoveTemp(HandlerFunc))));

		return *this;
	}

	/**
	 * Adds a message handler for the given type of messages (via TFunction object).
	 *
	 * It is legal to configure multiple handlers for the same message type. Each
	 * handler will be executed when a message of the specified type is received.
	 *
	 * This overload is used to register functions that are compatible with TFunction
	 * function objects, such as global and static functions, as well as lambdas.
	 *
	 * @param MessageType The type of messages to handle.
	 * @param Function The function object handling the messages.
	 * @return This instance (for method chaining).
	 * @see WithCatchall, WithHandler
	 */
	template<typename MessageType>
	FMessageEndpointBuilder& Handling(typename TFunctionMessageHandler<MessageType>::FuncType HandlerFunc)
	{
		Handlers.Add(MakeShareable(new TFunctionMessageHandler<MessageType>(MoveTemp(HandlerFunc))));

		return *this;
	}

	FMessageEndpointBuilder& NotificationHandling(FOnBusNotification&& InHandler)
	{
		OnNotification = MoveTemp(InHandler);
		return *this;
	}

	/**
	 * Configures the endpoint to receive messages on any thread.
	 *
	 * By default, the builder initializes the message endpoint to receive on the
	 * current thread. Use this method to receive on any available thread instead.
	 *
	 * ThreadAny is the fastest way to receive messages. It should be used if the receiving
	 * code is completely thread-safe and if it is sufficiently fast. ThreadAny MUST NOT
	 * be used if the receiving code is not thread-safe. It also SHOULD NOT be used if the
	 * code includes time consuming operations, because it will block the message router,
	 * causing no other messages to be delivered in the meantime.
	 *
	 * @return This instance (for method chaining).
	 * @see ReceivingOnThread
	 */
	FMessageEndpointBuilder& ReceivingOnAnyThread()
	{
		RecipientThread = ENamedThreads::AnyThread;

		return *this;
	}

	/**
	 * Configured the endpoint to receive messages on a specific thread.
	 *
	 * By default, the builder initializes the message endpoint to receive on the
	 * current thread. Use this method to receive on a different thread instead.
	 *
	 * Also see the additional notes for ReceivingOnAnyThread().
	 *
	 * @param NamedThread The name of the thread to receive messages on.
	 * @return This instance (for method chaining).
	 * @see ReceivingOnAnyThread
	 */
	FMessageEndpointBuilder& ReceivingOnThread(ENamedThreads::Type NamedThread)
	{
		RecipientThread = NamedThread;

		return *this;
	}

	/**
	 * Disables the endpoint.
	 *
	 * @return This instance (for method chaining).
	 */
	FMessageEndpointBuilder& ThatIsDisabled()
	{
		Disabled = true;

		return *this;
	}

	/**
	 * Adds a message handler for the given type of messages (via raw function pointers).
	 *
	 * It is legal to configure multiple handlers for the same message type. Each
	 * handler will be executed when a message of the specified type is received.
	 *
	 * This overload is used to register raw class member functions.
	 *
	 * @param HandlerType The type of the object handling the messages.
	 * @param MessageType The type of messages to handle.
	 * @param Handler The class handling the messages.
	 * @param HandlerFunc The class function handling the messages.
	 * @return This instance (for method chaining).
	 * @see WithHandler
	 */
	template<typename HandlerType>
	FMessageEndpointBuilder& WithCatchall(HandlerType* Handler, typename TRawMessageCatchall<HandlerType>::FuncType HandlerFunc)
	{
		Handlers.Add(MakeShareable(new TRawMessageCatchall<HandlerType>(Handler, MoveTemp(HandlerFunc))));

		return *this;
	}

	/**
	 * Adds a message handler for the given type of messages (via TFunction object).
	 *
	 * It is legal to configure multiple handlers for the same message type. Each
	 * handler will be executed when a message of the specified type is received.
	 *
	 * This overload is used to register functions that are compatible with TFunction
	 * function objects, such as global and static functions, as well as lambdas.
	 *
	 * @param MessageType The type of messages to handle.
	 * @param Function The function object handling the messages.
	 * @return This instance (for method chaining).
	 * @see WithHandler
	 */
	FMessageEndpointBuilder& WithCatchall(FFunctionMessageCatchall::FuncType HandlerFunc)
	{
		Handlers.Add(MakeShareable(new FFunctionMessageCatchall(MoveTemp(HandlerFunc))));

		return *this;
	}

	/**
	 * Registers a message handler with the endpoint.
	 *
	 * It is legal to configure multiple handlers for the same message type. Each
	 * handler will be executed when a message of the specified type is received.
	 *
	 * @param Handler The handler to add.
	 * @return This instance (for method chaining).
	 * @see Handling, WithCatchall
	 */
	FMessageEndpointBuilder& WithHandler(const TSharedRef<IMessageHandler, ESPMode::ThreadSafe>& Handler)
	{
		Handlers.Add(Handler);

		return *this;
	}

	/**
	 * Enables the endpoint's message inbox.
	 *
	 * The inbox is disabled by default.
	 *
	 * @return This instance (for method chaining).
	 */
	FMessageEndpointBuilder& WithInbox()
	{
		InboxEnabled = true;

		return *this;
	}

public:

	/**
	 * Builds the message endpoint as configured.
	 *
	 * @return A new message endpoint, or nullptr if it couldn't be built.
	 */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> Build()
	{
		TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> Endpoint;
		TSharedPtr<IMessageBus, ESPMode::ThreadSafe> Bus = BusPtr.Pin();
		
		if (Bus.IsValid())
		{
			Endpoint = MakeShared<FMessageEndpoint, ESPMode::ThreadSafe>(Name, Bus.ToSharedRef(), Handlers, OnNotification);
			Bus->Register(Endpoint->GetAddress(), Endpoint.ToSharedRef());

			if (OnNotification.IsBound())
			{
				Bus->AddNotificationListener(Endpoint.ToSharedRef());
			}

			if (Disabled)
			{
				Endpoint->Disable();
			}

			if (InboxEnabled)
			{
				Endpoint->EnableInbox();
				Endpoint->SetRecipientThread(ENamedThreads::AnyThread);
			}
			else
			{
				Endpoint->SetRecipientThread(RecipientThread);
			}
		}

		return Endpoint;
	}

	/**
	 * Implicit conversion operator to build the message endpoint as configured.
	 *
	 * @return The message endpoint.
	 */
	operator TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe>()
	{
		return Build();
	}
	
private:

	/** Holds a reference to the message bus to attach to. */
	TWeakPtr<IMessageBus, ESPMode::ThreadSafe> BusPtr;

	/** Holds a flag indicating whether the endpoint should be disabled. */
	bool Disabled;

	/** Holds a delegate to invoke on disconnection event. */
	FOnBusNotification OnNotification;

	/** Holds the collection of message handlers to register. */
	TArray<TSharedPtr<IMessageHandler, ESPMode::ThreadSafe>> Handlers;

	/** Holds a flag indicating whether the inbox should be enabled. */
	bool InboxEnabled;

	/** Holds the endpoint's name (for debugging purposes). */
	FName Name;

	/** Holds the name of the thread on which to receive messages. */
	ENamedThreads::Type RecipientThread;
};
