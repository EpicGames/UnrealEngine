// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMessageContext.h"
#include "IMessageHandler.h"


/**
 * Template for catch-all handlers (via raw function pointers).
 *
 * @param HandlerType The type of the handler class.
 */
template<typename HandlerType>
class TRawMessageCatchall
	: public IMessageHandler
{
public:

	/** Type definition for function pointers that are compatible with this TRawMessageCatchall. */
	typedef void (HandlerType::*FuncType)(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>&);

public:

	/**
	 * Creates and initializes a new message handler.
	 *
	 * @param InHandler The object that will handle the messages.
	 * @param InFunc The object's message handling function.
	 */
	TRawMessageCatchall(HandlerType* InHandler, FuncType InFunc)
		: Handler(InHandler)
		, Func(InFunc)
	{
		check(InHandler != nullptr);
	}

	/** Virtual destructor. */
	~TRawMessageCatchall() = default;

public:

	//~ IMessageHandler interface
	
	virtual void HandleMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context) override
	{
		(Handler->*Func)(Context);
	}
	
private:

	/** Holds a pointer to the object handling the messages. */
	HandlerType* Handler;

	/** Holds a pointer to the actual handler function. */
	FuncType Func;
};


/**
 * Template for handlers of one specific message type (via raw function pointers).
 *
 * @param MessageType The type of message to handle.
 * @param HandlerType The type of the handler class.
 */
template<typename MessageType, typename HandlerType>
class TRawMessageHandler
	: public IMessageHandler
{
public:

	/** Type definition for function pointers that are compatible with this TRawMessageHandler. */
	typedef void (HandlerType::*FuncType)(const MessageType&, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>&);

public:

	/**
	 * Creates and initializes a new message handler.
	 *
	 * @param InHandler The object that will handle the messages.
	 * @param InFunc The object's message handling function.
	 */
	TRawMessageHandler(HandlerType* InHandler, FuncType InFunc)
		: Handler(InHandler)
		, Func(InFunc)
	{
		check(InHandler != nullptr);
	}

	/** Virtual destructor. */
	~TRawMessageHandler() = default;

public:

	//~ IMessageHandler interface
	
	virtual void HandleMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context) override
	{
		UStruct* Struct = MessageType::StaticStruct();
		if (Struct && Context->GetMessageTypePathName() == Struct->GetStructPathName())
		{
			(Handler->*Func)(*static_cast<const MessageType*>(Context->GetMessage()), Context);
		}	
	}
	
private:

	/** Holds a pointer to the object handling the messages. */
	HandlerType* Handler;

	/** Holds a pointer to the actual handler function. */
	FuncType Func;
};


/**
 * Implements a catch-all handlers (via function objects).
 */
class FFunctionMessageCatchall
	: public IMessageHandler
{
public:

	/** Type definition for function objects that are compatible with this TFunctionMessageHandler. */
	typedef TFunction<void(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>&)> FuncType;

public:

	/**
	 * Creates and initializes a new message handler.
	 *
	 * @param InFunc The object's message handling function.
	 */
	FFunctionMessageCatchall(FuncType InFunc)
		: Func(MoveTemp(InFunc))
	{ }

	/** Virtual destructor. */
	~FFunctionMessageCatchall() = default;

public:

	//~ IMessageHandler interface
	
	virtual void HandleMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context) override
	{
		Func(Context);
	}
	
private:

	/** Holds a pointer to the actual handler function. */
	FuncType Func;
};


/**
 * Template for handlers of one specific message type (via function objects).
 *
 * @param MessageType The type of message to handle.
 */
template<typename MessageType>
class TFunctionMessageHandler
	: public IMessageHandler
{
public:

	/** Type definition for function objects that are compatible with this TFunctionMessageHandler. */
	typedef TFunction<void(const MessageType&, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>&)> FuncType;

public:

	/**
	 * Creates and initializes a new message handler.
	 *
	 * @param InHandlerFunc The object's message handling function.
	 */
	TFunctionMessageHandler(FuncType InFunc)
		: Func(MoveTemp(InFunc))
	{ }

	/** Virtual destructor. */
	~TFunctionMessageHandler() = default;

public:

	//~ IMessageHandler interface
	
	virtual void HandleMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context) override
	{
		if (Context->GetMessageTypePathName() == MessageType::StaticStruct()->GetStructPathName())
		{
			Func(*static_cast<const MessageType*>(Context->GetMessage()), Context);
		}	
	}
	
private:

	/** Holds a pointer to the actual handler function. */
	FuncType Func;
};
